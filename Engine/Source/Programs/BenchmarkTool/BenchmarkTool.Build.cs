// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class BenchmarkTool : ModuleRules
{
	public BenchmarkTool(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");
		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("Projects");
	}
}
