// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class BenchmarkToolTarget : TargetRules
{
	public BenchmarkToolTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LaunchModuleName = "BenchmarkTool";

		bBuildDeveloperTools			= false;
		bBuildWithEditorOnlyData		= false;
		bCompileAgainstEngine			= false;
		bCompileAgainstCoreUObject		= false;
		bCompileAgainstApplicationCore	= false;
		bIsBuildingConsoleApplication	= true;
	}
}
