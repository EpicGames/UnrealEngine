// Copyright Epic Games, Inc. All Rights Reserved.


#include "BenchmarkTool.h"
#include "Memory/MemoryArena.h"

#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"
#include "Misc/QueuedThreadPoolWrapper.h"
#include "RequiredProgramMainCPPInclude.h"
#include <locale.h>
#include <iterator>
#include <atomic>

DEFINE_LOG_CATEGORY_STATIC(LogBenchmarkTool, Log, All);

IMPLEMENT_APPLICATION(BenchmarkTool, "BenchTool");

//////////////////////////////////////////////////////////////////////////

class alignas(PLATFORM_CACHE_LINE_SIZE) BenchmarkState
{
public:
	struct BenchmarkIterator;

	BenchmarkState() = default;

	FORCEINLINE void SetIterationCount(int InIterationCount) { IterationCount = InIterationCount; }

	FORCEINLINE BenchmarkIterator begin();
	FORCEINLINE BenchmarkIterator end();

private:
	int		IterationCount = 1000;
};

struct BenchmarkState::BenchmarkIterator
{
public:
	BenchmarkIterator() = default;

	FORCEINLINE BenchmarkIterator(BenchmarkState* InState, int IterationCount)
	:	State(InState)
	,	Counter(IterationCount)
	{
	}

	FORCEINLINE BenchmarkIterator& operator++() { --Counter; return *this; }

	// This always assumes it compares to an end iterator
	FORCEINLINE bool operator!=(const BenchmarkIterator& Rhs)
	{
		if (Counter == 0)
		{
			return false;
		}

		return true;
	}

	// Let's just pretend we're an actual iterator

	struct Dummy {};
	typedef std::forward_iterator_tag	iterator_category;
	typedef Dummy						value_type;
	typedef Dummy						reference;
	typedef Dummy						pointer;
	typedef std::ptrdiff_t				difference_type;

	Dummy operator*() const { return Dummy(); }

private:
	BenchmarkState* State = nullptr;
	int				Counter = 0;
};

BenchmarkState::BenchmarkIterator BenchmarkState::begin() 
{ 
	return BenchmarkIterator(this, IterationCount); 
}

BenchmarkState::BenchmarkIterator BenchmarkState::end()
{
	return BenchmarkIterator();
}

//////////////////////////////////////////////////////////////////////////

FORCENOINLINE void UseCharPointer(char const volatile*) {}

//////////////////////////////////////////////////////////////////////////

typedef void(BenchFunction)(BenchmarkState&);

class Benchmark
{
public:
	Benchmark(const TCHAR* InName) : Name(InName) 
	{
	}

	virtual ~Benchmark() = default;

	Benchmark(const Benchmark&) = delete;
	Benchmark& operator=(const Benchmark&) = delete;

	virtual void DoRun(BenchmarkState& State)
	{
		UE_LOGF(LogBenchmarkTool, Log, "Running '%ls'...", *this->Name);

		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*this->Name);
		const uint64 StartTime = FPlatformTime::Cycles64();

		Run(State);

		Duration = FPlatformTime::Cycles64() - StartTime;
	}

	virtual Benchmark* Iterations(uint64 InIterationCount)	{ IterationCount = InIterationCount; return this; }
	virtual Benchmark* Threads(uint16 ThreadCount)			{ ThreadCounts.Add(ThreadCount); return this; }

	static Benchmark* RegisterBenchmarkInternal(Benchmark* InBenchmark);

protected:
	FString			Name;
	uint64			IterationCount = 0;
	TArray<uint16>	ThreadCounts;
	uint64			Duration = 0;			// This is in Cycles64 units

	friend class BenchmarkRegistry;

private:
	virtual void Run(BenchmarkState& State) = 0;
};

class BenchmarkFixture : public Benchmark
{
public:
	virtual void SetUp(BenchmarkState& State)
	{
	}

	virtual void TearDown(BenchmarkState& State)
	{
	}

	virtual void DoRun(BenchmarkState& State) override
	{
		SetUp(State);
		BenchmarkCase(State);
		TearDown(State);
	}

protected:
	virtual void BenchmarkCase(BenchmarkState&) = 0;
};

//////////////////////////////////////////////////////////////////////////

class BenchmarkReporter
{
public:
	BenchmarkReporter() = default;
	virtual ~BenchmarkReporter() = default;

	BenchmarkReporter(const BenchmarkReporter&) = delete;
	BenchmarkReporter& operator=(const BenchmarkReporter&) = delete;

	struct Run
	{
		FString	Name;
		uint64	IterationCount = 0;
		double	DurationMs = 0;
	};

	virtual void Start() {};
	virtual void ReportRuns(const TArray<Run>& Runs) = 0;
	virtual void Finalize() {};

private:

};

//////////////////////////////////////////////////////////////////////////

class BenchmarkRegistry
{
public:
	static BenchmarkRegistry& Get()
	{
		static BenchmarkRegistry Instance;
		return Instance;
	}

	Benchmark* Register(Benchmark* InBenchmark)
	{
		TUniquePtr<Benchmark> Bench(InBenchmark);
		Benchmarks.Add(MoveTemp(Bench));

		return InBenchmark;
	}

	void RunBenchmarks()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RunBenchmarks);
		TArray<BenchmarkReporter::Run> RunResults;
		RunResults.Reserve(Benchmarks.Num());

		FString BenchName;
		FParse::Value(FCommandLine::Get(), TEXT("-Benchmark="), BenchName);
		for (auto& Bench : Benchmarks)
		{
			if (BenchName.Len() > 0 && Bench->Name.Find(BenchName) == INDEX_NONE)
			{
				continue;
			}

			BenchmarkState State;
			State.SetIterationCount(Bench->IterationCount);

			Bench->DoRun(State);

			BenchmarkReporter::Run& RunResult = *new(RunResults) BenchmarkReporter::Run;

			RunResult.Name				= Bench->Name;
			RunResult.IterationCount	= Bench->IterationCount;
			RunResult.DurationMs		= FPlatformTime::ToMilliseconds64(Bench->Duration);
		}

		for (BenchmarkReporter::Run& Line : RunResults)
		{
			UE_LOGF(LogBenchmarkTool, Display, 
				"%-30ls %10ld iterations took %5ld ms (%f us/iteration)",
				*Line.Name,
				Line.IterationCount,
				(uint64)Line.DurationMs,
				Line.DurationMs * 1000. / Line.IterationCount);
		}
	}

	TArray<TUniquePtr<Benchmark>>	Benchmarks;
};

Benchmark* Benchmark::RegisterBenchmarkInternal(Benchmark* InBenchmark)
{
	return BenchmarkRegistry::Get().Register(InBenchmark);
}

class FunctionBenchmark : public Benchmark
{
public:
	FunctionBenchmark(const TCHAR* Name, BenchFunction* InFunction) 
	:	Benchmark(Name)
	,	Function(InFunction)
	{
	}

	virtual void Run(BenchmarkState& State) override
	{
		Function(State);
	}

private:
	BenchFunction*	Function = nullptr;
};

//////////////////////////////////////////////////////////////////////////

class ConsoleReporter : public BenchmarkReporter
{
public:
	ConsoleReporter()
	{
	}

	~ConsoleReporter()
	{
	}

	virtual void ReportRuns(const TArray<Run>& Runs) override
	{
	}

private:
};

//////////////////////////////////////////////////////////////////////////
//
// Benchmark macros
//

#if defined(__COUNTER__) && (__COUNTER__ + 1 == __COUNTER__ + 0)
#	define UE_BENCHMARK_UID __COUNTER__
#else
#	define UE_BENCHMARK_UID __LINE__
#endif

#define UE_BENCHMARK_NAME_(Name)		UE_BENCHMARK_CONCAT_(_benchmark_, UE_BENCHMARK_UID, Name)
#define UE_BENCHMARK_CONCAT_(a, b, c)	UE_BENCHMARK_CONCAT2_(a, b, c)
#define UE_BENCHMARK_CONCAT2_(a, b, c)	a##b##c

#define UE_BENCHMARK_DECLARE_(n)		static /*[[unused]]*/ ::Benchmark* UE_BENCHMARK_NAME_(n)

#define UE_BENCHMARK(n)					UE_BENCHMARK_DECLARE_(n) = (::Benchmark::RegisterBenchmarkInternal(new ::FunctionBenchmark(TEXT(#n), n)))

#define UE_BENCHMARK_CAPTURE(Func, Name, ...)	UE_BENCHMARK_DECLARE_(Func) = (::Benchmark::RegisterBenchmarkInternal(new ::FunctionBenchmark(TEXT(#Func "/" #Name), [](::BenchmarkState& State) { Func(State, __VA_ARGS__); })))

//////////////////////////////////////////////////////////////////////////

#if defined(_MSC_VER)
template <class T>
FORCEINLINE void DoNotOptimize(const T& Value) 
{
	UseCharPointer(&reinterpret_cast<char const volatile&>(Value));
	_ReadWriteBarrier();
}

inline FORCENOINLINE void ClobberMemory() { _ReadWriteBarrier(); }
#else
template <class T>
FORCEINLINE void DoNotOptimize(const T& Value) 
{
	/* TODO */
  UseCharPointer(&reinterpret_cast<char const volatile&>(Value));
}
inline FORCENOINLINE void ClobberMemory() { /* TODO */ }
#endif


//////////////////////////////////////////////////////////////////////////

#if UE_WITH_ARENAMAP
void BM_MapPtrToArena(BenchmarkState& State)
{
	FArenaMap::SetRangeToArena(0, 32ull * 1024 * 1024 * 1024, nullptr);
	FArenaMap::ClearRange(0, 32ull * 1024 * 1024 * 1024);

	int i = 0;

	for (auto _ : State)
	{
		FMemoryArena* Arena = FArenaMap::MapPtrToArena(reinterpret_cast<void*>(i++));
	}
}

UE_BENCHMARK(BM_MapPtrToArena)->Iterations(100000);
UE_BENCHMARK(BM_MapPtrToArena)->Iterations(1000000);
UE_BENCHMARK(BM_MapPtrToArena)->Iterations(10000000);
UE_BENCHMARK(BM_MapPtrToArena)->Iterations(100000000);
UE_BENCHMARK(BM_MapPtrToArena)->Iterations(1000000000);
#endif

void BM_NoOp(BenchmarkState& State)
{
	for (auto _ : State)
	{
	}
}

void BM_NoOp(BenchmarkState& State, int Count)
{
	for (auto _ : State)
	{
		for (int i = 0; i < Count; ++i)
		{
		}
	}
}

void BM_CritSecLoad(BenchmarkState& State)
{
	FCriticalSection Csec;
	int A = 0;

	for (auto _ : State)
	{
		Csec.Lock();
		int C = A;
		Csec.Unlock();
		DoNotOptimize(C);
	}
}

void BM_TAtomic(BenchmarkState& State)
{
	TAtomic<int> A;

	for (auto _ : State)
	{
		int C = A.Load();
		DoNotOptimize(C);
	}
}

void BM_TAtomicRelaxed(BenchmarkState& State)
{
	TAtomic<int> A;

	for (auto _ : State)
	{
		int C = A.Load(EMemoryOrder::Relaxed);
		DoNotOptimize(C);
	}
}

void BM_TAtomicStore(BenchmarkState& State)
{
	TAtomic<int> A;

	int i = 0;

	for (auto _ : State)
	{
		A.Store(i++);
		DoNotOptimize(A);
	}
}

void BM_TAtomicStoreRelaxed(BenchmarkState& State)
{
	TAtomic<int> A;

	int i = 0;

	for (auto _ : State)
	{
		A.Store(i++, EMemoryOrder::Relaxed);
		DoNotOptimize(A);
	}
}

void BM_StdAtomic(BenchmarkState& State)
{
	std::atomic<int> A;

	for (auto _ : State)
	{
		int C = A.load();
		DoNotOptimize(C);
	}
}

void BM_StdAtomicRelaxed(BenchmarkState& State)
{
	std::atomic<int> A;

	for (auto _ : State)
	{
		int C = A.load(std::memory_order_relaxed);
		DoNotOptimize(C);
	}
}

void BM_StdAtomicStore(BenchmarkState& State)
{
	std::atomic<int> A;

	int i = 0;

	for (auto _ : State)
	{
		A.store(i++);
		DoNotOptimize(A);
	}
}

void BM_StdAtomicStoreRelaxed(BenchmarkState& State)
{
	std::atomic<int> A;

	int i = 0;

	for (auto _ : State)
	{
		A.store(i++, std::memory_order_relaxed);
		DoNotOptimize(A);
	}
}

UE_BENCHMARK(BM_NoOp)->Iterations(100000000);
UE_BENCHMARK_CAPTURE(BM_NoOp, 1000, 1000)->Iterations(100000000);

UE_BENCHMARK(BM_CritSecLoad)->Iterations(100000000);

UE_BENCHMARK(BM_TAtomic)->Iterations(100000000);
UE_BENCHMARK(BM_TAtomicRelaxed)->Iterations(100000000);

UE_BENCHMARK(BM_TAtomicStore)->Iterations(100000000);
UE_BENCHMARK(BM_TAtomicStoreRelaxed)->Iterations(100000000);

UE_BENCHMARK(BM_StdAtomic)->Iterations(100000000);
UE_BENCHMARK(BM_StdAtomicRelaxed)->Iterations(100000000);

UE_BENCHMARK(BM_StdAtomicStore)->Iterations(100000000);
UE_BENCHMARK(BM_StdAtomicStoreRelaxed)->Iterations(100000000);

//////////////////////////////////////////////////////////////////////////
//
// Basic tests to measure uncontended RWLock/Critical section performance
//

void BM_ReadWriteLock_ReadLock(BenchmarkState& State)
{
	FRWLock Lock;

	for (auto _ : State)
	{
		Lock.ReadLock();
		Lock.ReadUnlock();
	}
}

void BM_ReadWriteLock_WriteLock(BenchmarkState& State)
{
	FRWLock Lock;

	for (auto _ : State)
	{
		Lock.WriteLock();
		Lock.WriteUnlock();
	}
}

void BM_CriticalSection(BenchmarkState& State)
{
	FCriticalSection Lock;

	for (auto _ : State)
	{
		Lock.Lock();
		Lock.Unlock();
	}
}

UE_BENCHMARK(BM_ReadWriteLock_ReadLock)->Iterations(10000000);
UE_BENCHMARK(BM_ReadWriteLock_ReadLock)->Iterations(100000000);
UE_BENCHMARK(BM_ReadWriteLock_WriteLock)->Iterations(10000000);
UE_BENCHMARK(BM_ReadWriteLock_WriteLock)->Iterations(100000000);
UE_BENCHMARK(BM_CriticalSection)->Iterations(10000000);
UE_BENCHMARK(BM_CriticalSection)->Iterations(100000000);

//////////////////////////////////////////////////////////////////////////

struct DummyShared
{
	int _ = 0;
};

void BM_TSharedPtr(BenchmarkState& State)
{
	for (auto _ : State)
	{
		TSharedPtr<DummyShared, ESPMode::ThreadSafe> Shared = MakeShared<DummyShared, ESPMode::ThreadSafe>();
		DoNotOptimize(Shared);
	}
}

void BM_TSharedPtrAssign(BenchmarkState& State)
{
	TSharedPtr<DummyShared, ESPMode::ThreadSafe> Shared = MakeShared<DummyShared, ESPMode::ThreadSafe>();

	for (auto _ : State)
	{
		auto Shared2 = Shared;
		DoNotOptimize(Shared2);
	}
}

void BM_TSharedPtr_NoTS(BenchmarkState& State)
{
	for (auto _ : State)
	{
		auto Shared = MakeShared<DummyShared, ESPMode::NotThreadSafe>();
		DoNotOptimize(Shared);
	}
}

void BM_TSharedPtrAssign_NoTS(BenchmarkState& State)
{
	auto Shared = MakeShared<DummyShared, ESPMode::NotThreadSafe>();

	for (auto _ : State)
	{
		auto Shared2 = Shared;
		DoNotOptimize(Shared2);
	}
}

struct DummyRefCount : public FRefCountedObject
{
	int _ = 0;
};

void BM_TRefCountPtr(BenchmarkState& State)
{
	for (auto _ : State)
	{
		TRefCountPtr<DummyRefCount> RefCount = new DummyRefCount();
		DoNotOptimize(RefCount);
	}
}

void BM_TRefCountAssign(BenchmarkState& State)
{
	TRefCountPtr<DummyRefCount> RefCount = new DummyRefCount();

	for (auto _ : State)
	{
		TRefCountPtr<DummyRefCount> Ref2 = RefCount;
		DoNotOptimize(Ref2);
	}
}

void BM_Scheduling_TaskGraphOverhead(BenchmarkState& State)
{
	FEvent* LastEvent = FPlatformProcess::GetSynchEventFromPool(true);
	FEvent* WaitEvent = FPlatformProcess::GetSynchEventFromPool(true);

	TAtomic<uint32> Count(0);
	for (auto _ : State)
	{
		Count++;
		FFunctionGraphTask::CreateAndDispatchWhenReady(
			[WaitEvent, LastEvent, &Count](ENamedThreads::Type CurrentThread, const FGraphEventRef&CompletionGraphEvent)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Task);

				// Stall the tasks so we can benchmark the queuing code in AddQueuedWork.
				// Otherwise, most threads will be able to execute as fast as the queuing
				// happens and the dispatching will occur directly on each thread, 
				// exercising a different code path.
				WaitEvent->Wait();

				if (--Count == 0)
				{
					LastEvent->Trigger();
				}
			},
			QUICK_USE_CYCLE_STAT(BM_Scheduling_TaskGraphOverhead, STATGROUP_ThreadPoolAsyncTasks),
			nullptr,
			ENamedThreads::AnyThread
			);
	}

	// Unstall the task processing so we can properly exercise the code
	// path where all the threads need to pick another job to process.
	WaitEvent->Trigger();

	// Wait until the last task has executed
	LastEvent->Wait();
	
	FPlatformProcess::ReturnSynchEventToPool(WaitEvent);
	FPlatformProcess::ReturnSynchEventToPool(LastEvent);
}

void BM_Scheduling_ThreadPoolOverhead_Impl(BenchmarkState& State, FQueuedThreadPool* ThreadPool)
{
	FEvent* LastEvent = FPlatformProcess::GetSynchEventFromPool(true);
	FEvent* WaitEvent = FPlatformProcess::GetSynchEventFromPool(true);

	TAtomic<uint32> Count(0);
	for (auto _ : State)
	{
		Count++;
		AsyncPool(
			*ThreadPool,
			[WaitEvent, LastEvent, &Count]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Task);

				// Stall the tasks so we can benchmark the queuing code in AddQueuedWork.
				// Otherwise, most threads will be able to execute as fast as the queuing
				// happens and the dispatching will occur directly on each thread, 
				// exercising a different code path.
				WaitEvent->Wait();

				if (--Count == 0)
				{
					LastEvent->Trigger();
				}
			}
		);
	}

	// Unstall the task processing so we can properly exercise the code
	// path where all the threads need to pick another job to process.
	WaitEvent->Trigger();

	// Wait until the last task has executed
	LastEvent->Wait();
	
	FPlatformProcess::ReturnSynchEventToPool(WaitEvent);
	FPlatformProcess::ReturnSynchEventToPool(LastEvent);
}

// This test is probably only meaningful when comparing relative speed
// of threadpool implementations and to profile the current one.
void BM_Scheduling_ThreadPoolOverhead(BenchmarkState& State)
{
	TUniquePtr<FQueuedThreadPool> ThreadPool(FQueuedThreadPool::Allocate());
	check(ThreadPool && ThreadPool->Create(FPlatformMisc::NumberOfCores()));

	BM_Scheduling_ThreadPoolOverhead_Impl(State, ThreadPool.Get());
}

// This test is probably only meaningful when comparing relative speed
// of threadpool implementations and to profile the current one.
void BM_Scheduling_ThreadPoolWrapperOverhead(BenchmarkState& State)
{
	TUniquePtr<FQueuedThreadPool> ThreadPool(FQueuedThreadPool::Allocate());
	check(ThreadPool && ThreadPool->Create(FPlatformMisc::NumberOfCores()));

	FQueuedThreadPoolWrapper ThreadPoolWrapper(ThreadPool.Get());

	BM_Scheduling_ThreadPoolOverhead_Impl(State, &ThreadPoolWrapper);
}

// This test is probably only meaningful when comparing relative speed
// of threadpool implementations and to profile the current one.
void BM_Scheduling_ThreadPoolLowLevelWrapperOverhead(BenchmarkState& State)
{
	TUniquePtr<FQueuedThreadPool> ThreadPool = MakeUnique<FQueuedLowLevelThreadPool>();
	ThreadPool->Create(0);
	BM_Scheduling_ThreadPoolOverhead_Impl(State, ThreadPool.Get());
}

UE_BENCHMARK(BM_TSharedPtr)->Iterations(100000000);
UE_BENCHMARK(BM_TRefCountPtr)->Iterations(100000000);
UE_BENCHMARK(BM_TSharedPtr_NoTS)->Iterations(100000000);
UE_BENCHMARK(BM_TSharedPtrAssign)->Iterations(100000000);
UE_BENCHMARK(BM_TRefCountAssign)->Iterations(100000000);
UE_BENCHMARK(BM_TSharedPtrAssign_NoTS)->Iterations(100000000);

// You can compare all scheduling tasks by using this command line -Benchmark=BM_Scheduling
// You can also test scalability of the different schedulers by adding -corelimit= to the command line
UE_BENCHMARK(BM_Scheduling_ThreadPoolOverhead)->Iterations(100000);
UE_BENCHMARK(BM_Scheduling_ThreadPoolWrapperOverhead)->Iterations(100000);
UE_BENCHMARK(BM_Scheduling_TaskGraphOverhead)->Iterations(100000);
UE_BENCHMARK(BM_Scheduling_ThreadPoolLowLevelWrapperOverhead)->Iterations(100000);

//////////////////////////////////////////////////////////////////////////

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	GEngineLoop.PreInit(ArgC, ArgV);
	BenchmarkRegistry::Get().RunBenchmarks();

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}